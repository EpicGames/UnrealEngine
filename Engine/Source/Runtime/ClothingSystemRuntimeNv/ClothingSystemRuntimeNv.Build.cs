// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClothingSystemRuntimeNv : ModuleRules
{
	public ClothingSystemRuntimeNv(ReadOnlyTargetRules Target) : base(Target)
	{
        SetupModulePhysicsSupport(Target);

        PublicIncludePathModuleNames.Add("ClothingSystemRuntimeInterface");

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "ClothingSystemRuntimeInterface",
                "ClothingSystemRuntimeCommon"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
            }
        );
    }
}
