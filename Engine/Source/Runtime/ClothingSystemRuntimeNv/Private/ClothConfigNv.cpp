// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothConfigNv.h"
#include "ClothConfig_Legacy.h"
#include "ClothConfigNvCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothConfigNv)

FClothConstraintSetupNv::FClothConstraintSetupNv()
	: Stiffness(1.0f)
	, StiffnessMultiplier(1.0f)
	, StretchLimit(1.0f)
	, CompressionLimit(1.0f)
{}

void FClothConstraintSetupNv::MigrateFrom(const FClothConstraintSetup_Legacy& Setup)
{
	Stiffness = Setup.Stiffness;
	StiffnessMultiplier = Setup.StiffnessMultiplier;
	StretchLimit = Setup.StretchLimit;
	CompressionLimit = Setup.CompressionLimit;
}

void FClothConstraintSetupNv::MigrateTo(FClothConstraintSetup_Legacy& Setup) const
{
	Setup.Stiffness = Stiffness;
	Setup.StiffnessMultiplier = StiffnessMultiplier;
	Setup.StretchLimit = StretchLimit;
	Setup.CompressionLimit = CompressionLimit;
}

UClothConfigNv::UClothConfigNv()
	: ClothingWindMethod(EClothingWindMethodNv::Legacy)
	, SelfCollisionRadius(0.0f)
	, SelfCollisionStiffness(0.0f)
	, SelfCollisionCullScale(1.0f)
	, Damping(0.4f)
	, Friction(0.1f)
	, WindDragCoefficient(0.02f/100.0f)
	, WindLiftCoefficient(0.02f/100.0f)
	, LinearDrag(0.2f)
	, AngularDrag(0.2f)
	, LinearInertiaScale(1.0f)
	, AngularInertiaScale(1.0f)
	, CentrifugalInertiaScale(1.0f)
	, SolverFrequency(120.0f)
	, StiffnessFrequency(100.0f)
	, GravityScale(1.0f)
	, GravityOverride(FVector::ZeroVector)
	, bUseGravityOverride(false)
	, TetherStiffness(1.0f)
	, TetherLimit(1.0f)
	, CollisionThickness(1.0f)
	, AnimDriveSpringStiffness(1.0f)
	, AnimDriveDamperStiffness(1.0f)
{}

void UClothConfigNv::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FClothConfigNvCustomVersion::GUID);
}

void UClothConfigNv::PostLoad()
{
	Super::PostLoad();

	const int32 ClothingCustomVersion = GetLinkerCustomVersion(FClothConfigNvCustomVersion::GUID);
	
	if (ClothingCustomVersion < FClothConfigNvCustomVersion::DeprecateLegacyStructureAndEnum)
	{
		ClothingWindMethod = static_cast<EClothingWindMethodNv>(WindMethod_DEPRECATED);
		VerticalConstraint.MigrateFrom(VerticalConstraintConfig_DEPRECATED);
		HorizontalConstraint.MigrateFrom(HorizontalConstraintConfig_DEPRECATED);
		BendConstraint.MigrateFrom(BendConstraintConfig_DEPRECATED);
		ShearConstraint.MigrateFrom(ShearConstraintConfig_DEPRECATED);
	}
}

void UClothConfigNv::MigrateFrom(const FClothConfig_Legacy& ClothConfig)
{
	ClothingWindMethod = static_cast<EClothingWindMethodNv>(ClothConfig.WindMethod);
	VerticalConstraint.MigrateFrom(ClothConfig.VerticalConstraintConfig);
	HorizontalConstraint.MigrateFrom(ClothConfig.HorizontalConstraintConfig);
	BendConstraint.MigrateFrom(ClothConfig.BendConstraintConfig);
	ShearConstraint.MigrateFrom(ClothConfig.ShearConstraintConfig);
	SelfCollisionRadius = ClothConfig.SelfCollisionRadius;
	SelfCollisionStiffness = ClothConfig.SelfCollisionStiffness;
	SelfCollisionCullScale = ClothConfig.SelfCollisionCullScale;
	Damping = ClothConfig.Damping;
	Friction = ClothConfig.Friction;
	WindDragCoefficient = ClothConfig.WindDragCoefficient;
	WindLiftCoefficient = ClothConfig.WindLiftCoefficient;
	LinearDrag = ClothConfig.LinearDrag;
	AngularDrag = ClothConfig.AngularDrag;
	LinearInertiaScale = ClothConfig.LinearInertiaScale;
	AngularInertiaScale = ClothConfig.AngularInertiaScale;
	CentrifugalInertiaScale = ClothConfig.CentrifugalInertiaScale;
	SolverFrequency = ClothConfig.SolverFrequency;
	StiffnessFrequency = ClothConfig.StiffnessFrequency;
	GravityScale = ClothConfig.GravityScale;
	GravityOverride = ClothConfig.GravityOverride;
	bUseGravityOverride = ClothConfig.bUseGravityOverride;
	TetherStiffness = ClothConfig.TetherStiffness;
	TetherLimit = ClothConfig.TetherLimit;
	CollisionThickness = ClothConfig.CollisionThickness;
	AnimDriveSpringStiffness = ClothConfig.AnimDriveSpringStiffness;
	AnimDriveDamperStiffness = ClothConfig.AnimDriveDamperStiffness;
}

bool UClothConfigNv::MigrateTo(FClothConfig_Legacy& ClothConfig) const
{
	ClothConfig.WindMethod = static_cast<EClothingWindMethod_Legacy>(ClothingWindMethod);
	VerticalConstraint.MigrateTo(ClothConfig.VerticalConstraintConfig);
	HorizontalConstraint.MigrateTo(ClothConfig.HorizontalConstraintConfig);
	BendConstraint.MigrateTo(ClothConfig.BendConstraintConfig);
	ShearConstraint.MigrateTo(ClothConfig.ShearConstraintConfig);
	ClothConfig.SelfCollisionRadius = SelfCollisionRadius;
	ClothConfig.SelfCollisionStiffness = SelfCollisionStiffness;
	ClothConfig.SelfCollisionCullScale = SelfCollisionCullScale;
	ClothConfig.Damping = Damping;
	ClothConfig.Friction = Friction;
	ClothConfig.WindDragCoefficient = WindDragCoefficient;
	ClothConfig.WindLiftCoefficient = WindLiftCoefficient;
	ClothConfig.LinearDrag = LinearDrag;
	ClothConfig.AngularDrag = AngularDrag;
	ClothConfig.LinearInertiaScale = LinearInertiaScale;
	ClothConfig.AngularInertiaScale = AngularInertiaScale;
	ClothConfig.CentrifugalInertiaScale = CentrifugalInertiaScale;
	ClothConfig.SolverFrequency = SolverFrequency;
	ClothConfig.StiffnessFrequency = StiffnessFrequency;
	ClothConfig.GravityScale = GravityScale;
	ClothConfig.GravityOverride = GravityOverride;
	ClothConfig.bUseGravityOverride = bUseGravityOverride;
	ClothConfig.TetherStiffness = TetherStiffness;
	ClothConfig.TetherLimit = TetherLimit;
	ClothConfig.CollisionThickness = CollisionThickness;
	ClothConfig.AnimDriveSpringStiffness = AnimDriveSpringStiffness;
	ClothConfig.AnimDriveDamperStiffness = AnimDriveDamperStiffness;

	return true;
}

