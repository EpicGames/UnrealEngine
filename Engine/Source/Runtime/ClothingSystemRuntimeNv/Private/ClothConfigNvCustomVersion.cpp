// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothConfigNvCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FClothConfigNvCustomVersion::GUID(0x9950B70E, 0xB41A4E17, 0xBBCCFA0D, 0x57817FD6);

// Register the custom version with core
FCustomVersionRegistration GRegisterClothConfigNvCustomVersion(FClothConfigNvCustomVersion::GUID, FClothConfigNvCustomVersion::LatestVersion, TEXT("ClothConfigNvVer"));
