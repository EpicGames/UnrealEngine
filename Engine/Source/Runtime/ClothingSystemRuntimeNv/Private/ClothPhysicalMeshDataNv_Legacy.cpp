// Copyright Epic Games, Inc. All Rights Reserved.
#include "ClothPhysicalMeshDataNv_Legacy.h"
#include "ClothPhysicalMeshData.h"  // For EWeightMapTargetCommon

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothPhysicalMeshDataNv_Legacy)

UClothPhysicalMeshDataNv_Legacy::UClothPhysicalMeshDataNv_Legacy()
{
	Super::RegisterFloatArray((uint32)EWeightMapTargetCommon::MaxDistance, &MaxDistances);
	Super::RegisterFloatArray((uint32)EWeightMapTargetCommon::BackstopDistance, &BackstopDistances);
	Super::RegisterFloatArray((uint32)EWeightMapTargetCommon::BackstopRadius, &BackstopRadiuses);
	Super::RegisterFloatArray((uint32)EWeightMapTargetCommon::AnimDriveStiffness, &AnimDriveMultipliers);
}

UClothPhysicalMeshDataNv_Legacy::~UClothPhysicalMeshDataNv_Legacy()
{}

