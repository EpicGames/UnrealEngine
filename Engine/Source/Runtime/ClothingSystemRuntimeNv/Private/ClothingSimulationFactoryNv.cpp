// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationFactoryNv.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingSimulationFactoryNv)

// Legacy support allowing the loading of older classes that may still reference old NvCloth simulation
// classes. These are no longer expected to work but should still load so the references can be safely changed.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
IClothingSimulationInterface* UClothingSimulationFactoryNv::CreateSimulation() const
{
	return nullptr;
}

void UClothingSimulationFactoryNv::DestroySimulation(IClothingSimulationInterface* InSimulation) const
{
}

bool UClothingSimulationFactoryNv::SupportsAsset(const UClothingAssetBase* InAsset) const
{
	return false;
}

bool UClothingSimulationFactoryNv::SupportsRuntimeInteraction() const
{
	return false;
}

UClothingSimulationInteractor* UClothingSimulationFactoryNv::CreateInteractor()
{
	return nullptr;
}

TArrayView<const TSubclassOf<UClothConfigBase>> UClothingSimulationFactoryNv::GetClothConfigClasses() const
{
	return TArrayView<const TSubclassOf<UClothConfigBase>>();
}

const UEnum* UClothingSimulationFactoryNv::GetWeightMapTargetEnum() const
{
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

