// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationInteractorNv.h"

#include "ClothingSimulationNv.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingSimulationInteractorNv)

// Stub implementations to allow older assets that reference this interactor to load safely so they can
// be updated to use the Chaos version in future.
void UClothingSimulationInteractorNv::PhysicsAssetUpdated()
{
}

void UClothingSimulationInteractorNv::ClothConfigUpdated()
{
}

void UClothingSimulationInteractorNv::Sync(IClothingSimulationInterface* InSimulation, const IClothingSimulationContext* InContext)
{
}

void UClothingSimulationInteractorNv::SetAnimDriveSpringStiffness(float InStiffness)
{
}

void UClothingSimulationInteractorNv::SetAnimDriveDamperStiffness(float InStiffness)
{
}
void UClothingSimulationInteractorNv::EnableGravityOverride(const FVector& InVector)
{
}

void UClothingSimulationInteractorNv::DisableGravityOverride()
{
}


