// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSystemRuntimeModuleNv.h"
#include "ClothingSimulationFactoryNv.h"
#include "Features/IModularFeatures.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"
#include "Modules/ModuleManager.h"

#include "NvClothSupport.h"


IMPLEMENT_MODULE(FClothingSystemRuntimeModuleNv, ClothingSystemRuntimeNv);

FClothingSystemRuntimeModuleNv::FClothingSystemRuntimeModuleNv()
{

}

void FClothingSystemRuntimeModuleNv::StartupModule()
{
}

void FClothingSystemRuntimeModuleNv::ShutdownModule()
{
}

nv::cloth::Factory* FClothingSystemRuntimeModuleNv::GetSoftwareFactory()
{
	return nullptr;
}

nv::cloth::ClothMeshQuadifier* FClothingSystemRuntimeModuleNv::GetMeshQuadifier()
{
	return nullptr;
}

TSubclassOf<UClothingSimulationFactory> FClothingSystemRuntimeModuleNv::GetClothingSimulationFactoryClass() const
{
	return nullptr;
}
