// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothConfig.h"
#include "ClothConfig_Legacy.h"
#include "CoreTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothConfigNv.generated.h"

class FArchive;
class UObject;

/** Container for a constraint setup, these can be horizontal, vertical, shear and bend. */
USTRUCT()
struct FClothConstraintSetupNv
{
	GENERATED_BODY()

	FClothConstraintSetupNv();

	// Migrate from the legacy FClothConstraintSetup structure.
	void MigrateFrom(const FClothConstraintSetup_Legacy& Setup);

	// Migrate to the legacy FClothConstraintSetup structure.
	void MigrateTo(FClothConstraintSetup_Legacy& Setup) const;

	// How stiff this constraint is, this affects how closely it will follow the desired position
	UPROPERTY(EditAnywhere, Category=Constraint)
	float Stiffness;

	// A multiplier affecting the above value
	UPROPERTY(EditAnywhere, Category = Constraint)
	float StiffnessMultiplier;

	// The hard limit on how far this constraint can stretch
	UPROPERTY(EditAnywhere, Category = Constraint)
	float StretchLimit;

	// The hard limit on how far this constraint can compress
	UPROPERTY(EditAnywhere, Category = Constraint)
	float CompressionLimit;
};

/** Cloth wind method. */
UENUM()
enum class EClothingWindMethodNv : uint8
{
	// Use legacy wind mode, where accelerations are modified directly by the simulation
	// with no regard for drag or lift
	Legacy,

	// Use updated wind calculation for NvCloth based solved taking into account
	// drag and lift, this will require those properties to be correctly set in
	// the clothing configuration
	Accurate
};

/** Holds initial, asset level config for clothing actors. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(MinimalAPI)
class UClothConfigNv : public UClothConfigCommon
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMENV_API UClothConfigNv();

	// Set up custom version serialization.
	CLOTHINGSYSTEMRUNTIMENV_API virtual void Serialize(FArchive& Ar) override;

	// Update the deprecated properties.
	CLOTHINGSYSTEMRUNTIMENV_API virtual void PostLoad() override;

	// Migrate from the legacy FClothConfig structure.
	CLOTHINGSYSTEMRUNTIMENV_API virtual void MigrateFrom(const FClothConfig_Legacy& ClothConfig) override;

	// Migrate to the legacy FClothConfig structure.
	CLOTHINGSYSTEMRUNTIMENV_API virtual bool MigrateTo(FClothConfig_Legacy& ClothConfig) const override;

	/** Return whether to pre-compute self collision data. */
	virtual bool NeedsSelfCollisionData() const override { return SelfCollisionRadius > 0.0f && SelfCollisionStiffness > 0.0f; }

	/** Return whether to pre-compute inverse masses. */
	virtual bool NeedsInverseMasses() const override { return true; }

	/** Return whether to pre-compute the long range attachment tethers. */
	virtual bool NeedsTethers() const override { return false; }

	// Return the collision radius required to calculate the self collision indices, or 0.f if self collision is disabled.
	virtual float GetSelfCollisionRadius() const override { return NeedsSelfCollisionData() ? SelfCollisionRadius * SelfCollisionCullScale : 0.0f; }

	// How wind should be processed, Accurate uses drag and lift to make the cloth react differently, legacy applies similar forces to all clothing without drag and lift (similar to APEX)
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	EClothingWindMethodNv ClothingWindMethod;

	// Constraint data for vertical constraints
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FClothConstraintSetupNv VerticalConstraint;

	// Constraint data for horizontal constraints
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FClothConstraintSetupNv HorizontalConstraint;

	// Constraint data for bend constraints
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FClothConstraintSetupNv BendConstraint;

	// Constraint data for shear constraints
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FClothConstraintSetupNv ShearConstraint;

	// Size of self collision spheres centered on each vert
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float SelfCollisionRadius;

	// Stiffness of the spring force that will resolve self collisions
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float SelfCollisionStiffness;

	/** 
	 * Scale to use for the radius of the culling checks for self collisions.
	 * Any other self collision body within the radius of this check will be culled.
	 * This helps performance with higher resolution meshes by reducing the number
	 * of colliding bodies within the cloth. Reducing this will have a negative
	 * effect on performance!
	 */
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (UIMin="0", ClampMin="0"))
	float SelfCollisionCullScale;

	// Damping of particle motion per-axis
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FVector Damping;

	// Friction of the surface when colliding
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float Friction;

	// Drag coefficient for wind calculations, higher values mean wind has more lateral effect on cloth
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float WindDragCoefficient;

	// Lift coefficient for wind calculations, higher values make cloth rise easier in wind
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float WindLiftCoefficient;

	// Drag applied to linear particle movement per-axis
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FVector LinearDrag;

	// Drag applied to angular particle movement, higher values should limit material bending (per-axis)
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	FVector AngularDrag;

	// Scale for linear particle inertia, how much movement should translate to linear motion (per-axis)
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (UIMin="0", UIMax="1", ClampMin="0", ClampMax="1"))
	FVector LinearInertiaScale;

	// Scale for angular particle inertia, how much movement should translate to angular motion (per-axis)
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FVector AngularInertiaScale;

	// Scale for centrifugal particle inertia, how much movement should translate to angular motion (per-axis)
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FVector CentrifugalInertiaScale;

	// Frequency of the position solver, lower values will lead to stretchier, bouncier cloth
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (UIMin = "30", UIMax = "240", ClampMin = "30", ClampMax = "1000"))
	float SolverFrequency;

	// Frequency for stiffness calculations, lower values will degrade stiffness of constraints
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float StiffnessFrequency;

	// Scale of gravity effect on particles
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (EditCondition = "!bUseGravityOverride"))
	float GravityScale;

	// Direct gravity override value
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (EditCondition = "bUseGravityOverride"))
	FVector GravityOverride;

	/** Use gravity override value vs gravity scale */
	UPROPERTY(EditAnywhere, Category = ClothConfig, meta = (InlineEditConditionToggle))
	bool bUseGravityOverride;

	// Scale for stiffness of particle tethers between each other
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float TetherStiffness;

	// Scale for the limit of particle tethers (how far they can separate)
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float TetherLimit;

	// 'Thickness' of the simulated cloth, used to adjust collisions
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float CollisionThickness;

	// Default spring stiffness for anim drive if an anim drive is in use
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float AnimDriveSpringStiffness;

	// Default damper stiffness for anim drive if an anim drive is in use
	UPROPERTY(EditAnywhere, Category = ClothConfig)
	float AnimDriveDamperStiffness;

	// Deprecated properties using old legacy structure and enum that couldn't be redirected
	UPROPERTY()
	EClothingWindMethod_Legacy WindMethod_DEPRECATED;
	UPROPERTY()
	FClothConstraintSetup_Legacy VerticalConstraintConfig_DEPRECATED;
	UPROPERTY()
	FClothConstraintSetup_Legacy HorizontalConstraintConfig_DEPRECATED;
	UPROPERTY()
	FClothConstraintSetup_Legacy BendConstraintConfig_DEPRECATED;
	UPROPERTY()
	FClothConstraintSetup_Legacy ShearConstraintConfig_DEPRECATED;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
