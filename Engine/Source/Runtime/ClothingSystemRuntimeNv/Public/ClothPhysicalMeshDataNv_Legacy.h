// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothPhysicalMeshDataBase_Legacy.h"
#include "Containers/Array.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothPhysicalMeshDataNv_Legacy.generated.h"

class UObject;

/**
 * Deprecated, use FClothPhysicalMeshData instead.
 * NV specific spatial simulation data for a mesh.
 */
UCLASS(MinimalAPI)
class UClothPhysicalMeshDataNv_Legacy: public UClothPhysicalMeshDataBase_Legacy
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMENV_API UClothPhysicalMeshDataNv_Legacy();
	CLOTHINGSYSTEMRUNTIMENV_API virtual ~UClothPhysicalMeshDataNv_Legacy();

	// The distance that each vertex can move away from its reference (skinned) position
	UPROPERTY()
	TArray<float> MaxDistances;

	// Distance along the plane of the surface that the particles can travel (separation constraint)
	UPROPERTY()
	TArray<float> BackstopDistances;

	// Radius of movement to allow for backstop movement
	UPROPERTY()
	TArray<float> BackstopRadiuses;

	// Strength of anim drive per-particle (spring driving particle back to skinned location
	UPROPERTY()
	TArray<float> AnimDriveMultipliers;
};
