// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UClothingAssetCommon;

// Base clothing actor just needs a link back to its parent asset, everything else defined by derived simulation
class FClothingActorBase
{
public:

	// Link back to parent asset
	UClothingAssetCommon* AssetCreatedFrom;
};
