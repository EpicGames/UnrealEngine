// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingSimulationFactory.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothingSimulationFactoryNv.generated.h"

class UClothConfigBase;
class UClothingAssetBase;
class UClothingSimulationInteractor;
class UEnum;
class UObject;
template <class TClass> class TSubclassOf;

class UE_DEPRECATED(5.1, "NvCloth simulation is no longer supported, UChaosClothingSimulationFactory should be used going forward.") UClothingSimulationFactoryNv;

UCLASS(MinimalAPI)
class UClothingSimulationFactoryNv final : public UClothingSimulationFactory
{
	GENERATED_BODY()
public:

	CLOTHINGSYSTEMRUNTIMENV_API virtual IClothingSimulationInterface* CreateSimulation() const override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual void DestroySimulation(IClothingSimulationInterface* InSimulation) const override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual bool SupportsAsset(const UClothingAssetBase* InAsset) const override;

	CLOTHINGSYSTEMRUNTIMENV_API virtual bool SupportsRuntimeInteraction() const override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual UClothingSimulationInteractor* CreateInteractor() override;

	CLOTHINGSYSTEMRUNTIMENV_API virtual TArrayView<const TSubclassOf<UClothConfigBase>> GetClothConfigClasses() const override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual const UEnum* GetWeightMapTargetEnum() const override;
};
