// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingSimulationInteractor.h"
#include "ClothingSimulationNv.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Math/MathFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothingSimulationInteractorNv.generated.h"

class FClothingSimulationContextNv;
class FClothingSimulationNv;
class IClothingSimulationContext;
class UObject;
struct FFrame;

// Command signature for handling synced command buffer
DECLARE_DELEGATE_TwoParams(NvInteractorCommand, FClothingSimulationNv*, FClothingSimulationContextNv*)

class UE_DEPRECATED(5.1, "NvCloth is no longer a supported clothing simlation provider, prefer the Chaos simulation moving forward.") UClothingSimulationInteractorNv;
UCLASS(BlueprintType, MinimalAPI)
class UClothingSimulationInteractorNv : public UClothingSimulationInteractor
{
	GENERATED_BODY()

public:

	// UClothingSimulationInteractor Interface
	CLOTHINGSYSTEMRUNTIMENV_API virtual void Sync(IClothingSimulationInterface* InSimulation, const IClothingSimulationContext* InContext) override;

	CLOTHINGSYSTEMRUNTIMENV_API virtual void PhysicsAssetUpdated() override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual void ClothConfigUpdated() override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual void SetAnimDriveSpringStiffness(float InStiffness) override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual void EnableGravityOverride(const FVector& InVector) override;
	CLOTHINGSYSTEMRUNTIMENV_API virtual void DisableGravityOverride() override;

	// TODO: These new functions are currently unimplemented
	virtual void SetNumIterations(int32 /*NumIterations*/) override {}
	virtual void SetMaxNumIterations(int32 /*MaxNumIterations*/) override {}
	virtual void SetNumSubsteps(int32 /*NumSubsteps*/) override {}
	//////////////////////////////////////////////////////////////////////////

	// Set the stiffness of the resistive damping force for anim drive
	UFUNCTION(BlueprintCallable, Category=ClothingSimulation)
	CLOTHINGSYSTEMRUNTIMENV_API void SetAnimDriveDamperStiffness(float InStiffness);

protected:

	virtual UClothingInteractor* CreateClothingInteractor() override { return nullptr; }

private:
};
