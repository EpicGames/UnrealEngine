// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingSimulationFactory.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"

namespace nv
{
	namespace cloth
	{
		class ClothMeshQuadifier;
		class Factory;
	}
}

class FClothingSystemRuntimeModuleNv : public IModuleInterface, public IClothingSimulationFactoryClassProvider
{

public:

	FClothingSystemRuntimeModuleNv();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	nv::cloth::Factory* GetSoftwareFactory();
	nv::cloth::ClothMeshQuadifier* GetMeshQuadifier();

	// IClothingSimulationFactoryClassProvider Interface
	virtual TSubclassOf<UClothingSimulationFactory> GetClothingSimulationFactoryClass() const override;
	//////////////////////////////////////////////////////////////////////////

private:

};
