// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.4, "NvClothIncludes.h is no longer useful.")

#include "CoreMinimal.h"
#include "EngineDefines.h"
